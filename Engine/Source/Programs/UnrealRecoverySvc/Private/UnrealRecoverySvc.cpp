// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertSettings.h"
#include "ConcertSyncServerLoop.h"
#include "ConcertLocalFileSharingService.h"

#include "Misc/CommandLine.h"
#include "RequiredProgramMainCPPInclude.h"

IMPLEMENT_APPLICATION(UnrealRecoverySvc, "UnrealRecoverySvc");

/**
 * Application entry point
 *
 * @param	ArgC	Command-line argument count
 * @param	ArgV	Argument strings
 */
INT32_MAIN_INT32_ARGC_TCHAR_ARGV()
{
	uint32 EditorProcessId = 0;

	FConcertSyncServerLoopInitArgs ServerLoopInitArgs;
	ServerLoopInitArgs.IdealFramerate = 30;
	ServerLoopInitArgs.SessionFlags = EConcertSyncSessionFlags::Default_DisasterRecoverySession;
	ServerLoopInitArgs.ServiceRole = TEXT("DisasterRecovery");
	ServerLoopInitArgs.ServiceFriendlyName = TEXT("Disaster Recovery Service");
	ServerLoopInitArgs.ServiceAutoArchiveSessionFilter.bIncludeIgnoredActivities = true;
	ServerLoopInitArgs.FileSharingService = MakeShared<FConcertLocalFileSharingService>(ServerLoopInitArgs.ServiceRole);
	ServerLoopInitArgs.bShowConsole = false;

	ServerLoopInitArgs.GetServerConfigFunc = [&EditorProcessId]() -> const UConcertServerConfig*
	{
		FParse::Value(FCommandLine::Get(), TEXT("-EDITORPID="), EditorProcessId);
		if (EditorProcessId)
		{
			UE_LOGF(LogSyncServer, Display, "Watching Editor process %d", EditorProcessId);
		}
		else
		{
			UE_LOGF(LogSyncServer, Error, "Invalid -EditorPID argument. Cannot continue!");
			return nullptr;
		}

		UConcertServerConfig* ServerConfig = IConcertSyncServerModule::Get().ParseServerSettings(FCommandLine::Get());
		ServerConfig->bAutoArchiveOnReboot = false; // Skip archiving, this can takes several minutes. It is more efficient to leave the live session as it and 'copy' it on demand (to restore).
		ServerConfig->bAutoArchiveOnShutdown = false; // Skip archiving, this can takes several minutes. It is more efficient to leave the live session as it and 'copy' it on demand (to restore).
		ServerConfig->EndpointSettings.RemoteEndpointTimeoutSeconds = 0; // Ensure the endpoints never time out (and are kept alive automatically by Concert).
		ServerConfig->bMountDefaultSessionRepository = false; // Let the client mount its own repository to support concurrent service and avoid them to access the same non-sharable database files.
		ServerConfig->AuthorizedClientKeys.Add(ServerConfig->ServerName); // The disaster recovery client is configured to use the unique server name as key to identify itself.
		return ServerConfig;
	};

	FTSTicker::GetCoreTicker().AddTicker(TEXT("CheckEditorHealth"), 1.0f, [&EditorProcessId](float)
	{
		if (!FPlatformProcess::IsApplicationRunning(EditorProcessId))
		{
			UE_LOGF(LogSyncServer, Warning, "Editor process %d lost! Requesting exit.", EditorProcessId);
			FPlatformMisc::RequestExit(false);
		}
		return true;
	});

	return ConcertSyncServerLoop(*FCommandLine::BuildFromArgV(nullptr, ArgC, ArgV, nullptr), ServerLoopInitArgs);
}
