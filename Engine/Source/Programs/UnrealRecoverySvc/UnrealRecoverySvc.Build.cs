// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class UnrealRecoverySvc : ModuleRules
{
	public UnrealRecoverySvc(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Concert",
				"ConcertServer",
				"ConcertSyncCore",
				"ApplicationCore",					// for LaunchEngineLoop.cpp dependency
				"Projects",							// for LaunchEngineLoop.cpp dependency
			}
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"Launch",
				"ConcertSyncServer",
			}
		);
	}
}
