// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.Collections.Generic;

public class UnrealRecoverySvcTarget : TargetRules
{
	public UnrealRecoverySvcTarget(TargetInfo Target) : base(Target)
	{
		Type = TargetType.Program;
		LinkType = TargetLinkType.Monolithic;
		LaunchModuleName = "UnrealRecoverySvc";

		// This app compiles against Core/CoreUObject, but not the Engine or Editor, so compile out Engine and Editor references from Core/CoreUObject
		bCompileAgainstCoreUObject = true;
		bCompileAgainstEngine = false;
		bBuildWithEditorOnlyData = false;

		// Enable Developer plugins (like Concert!)
		bCompileWithPluginSupport = true;
		bBuildDeveloperTools = true;

		// This app is a console application (sets entry point to main(), instead of WinMain())
		bIsBuildingConsoleApplication = true;
	}
}
